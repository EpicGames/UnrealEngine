{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Apex Destruction",
	"Description" : "APEX implementation of destruction",
	"Category" : "Physics",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name": "ApexDestruction",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "ApexDestructionEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	]
}