[CoreRedirects]

+ClassRedirects=(OldName="DestructibleActor", NewName="/Script/ApexDestruction.DestructibleActor")
+ClassRedirects=(OldName="DestructibleComponent", NewName="/Script/ApexDestruction.DestructibleComponent")
+ClassRedirects=(OldName="DestructibleMesh", NewName="/Script/ApexDestruction.DestructibleMesh")
+ClassRedirects=(OldName="DestructibleFractureSettings", NewName="/Script/ApexDestruction.DestructibleFractureSettings")