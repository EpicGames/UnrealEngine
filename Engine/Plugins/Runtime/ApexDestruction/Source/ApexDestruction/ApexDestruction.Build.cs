// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ApexDestruction : ModuleRules
{
	public ApexDestruction(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject", // @todo Mac: for some reason it's needed to link in debug on Mac
				"Engine",
//				"PhysX",
//				"APEX",
				"RHI",
				"RenderCore",
//				"ApexDestructionLib",
				"NavigationSystem",
            }
		);

        if (Target.Type == TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
					"EditorFramework",
                    "UnrealEd"
                }
            );
        }
        
        SetupModulePhysicsSupport(Target);
	}
}
