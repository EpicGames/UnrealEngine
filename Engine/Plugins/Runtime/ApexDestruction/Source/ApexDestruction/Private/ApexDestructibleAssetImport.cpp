// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ApexDestructibleAssetImport.cpp:

	Creation of an APEX NxDestructibleAsset from a binary buffer.
	SkeletalMesh creation from an APEX NxDestructibleAsset.

	SkeletalMesh creation largely based on FbxSkeletalMeshImport.cpp

=============================================================================*/

#include "ApexDestructibleAssetImport.h"
#include "Logging/LogMacros.h"

#if WITH_EDITOR

DEFINE_LOG_CATEGORY_STATIC(LogApexDestructibleAssetImport, Log, All);

#endif // WITH_EDITOR
