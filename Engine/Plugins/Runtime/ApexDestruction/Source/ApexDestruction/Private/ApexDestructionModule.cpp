// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApexDestructionModule.h"
#include "PhysicsCore.h"
#include "Modules/ModuleManager.h"
#include "PhysicsPublic.h"

#ifndef APEX_STATICALLY_LINKED
#define APEX_STATICALLY_LINKED	0
#endif

class UDestructibleComponent;

namespace nvidia
{
	namespace apex
	{
		class ModuleDestructible;
	}
}

class FApexDestructionModule : public IModuleInterface
{
private:
	FDelegateHandle OnPhysSceneInitHandle;
	FDelegateHandle OnPhysDispatchNotifications;

public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{	
		OnPhysSceneInitHandle = FPhysicsDelegates::OnPhysSceneInit.AddRaw(this, &FApexDestructionModule::OnInitPhys);
		OnPhysDispatchNotifications = FPhysicsDelegates::OnPhysDispatchNotifications.AddRaw(this, &FApexDestructionModule::OnDispatchNotifications);

		Singleton = this;

		UE_LOGF(LogPhysicsCore, Log, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.");
	}

	virtual void OnInitPhys(FPhysScene* PhysScene)
	{
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void OnDispatchNotifications(FPhysScene* PhysScene)
	{
	}

	void AddPendingDamageEvent(UDestructibleComponent* DestructibleComponent, const nvidia::apex::DamageEventReportData& DamageEvent)
	{
	}

	virtual void ShutdownModule() override
	{
		FPhysicsDelegates::OnPhysSceneInit.Remove(OnPhysSceneInitHandle);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static FApexDestructionModule& GetSingleton()
	{
		return *Singleton;
	}

	static FApexDestructionModule* Singleton;

};

FApexDestructionModule* FApexDestructionModule::Singleton = nullptr;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
IMPLEMENT_MODULE(FApexDestructionModule, ApexDestruction)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
