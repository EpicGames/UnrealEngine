// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DestructibleActor.cpp: ADestructibleActor methods.
=============================================================================*/


#include "DestructibleActor.h"
#include "DestructibleComponent.h"
#include "Engine/SkinnedAsset.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ADestructibleActor::ADestructibleActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DestructibleComponent = CreateDefaultSubobject<UDestructibleComponent>(TEXT("DestructibleComponent0"));
	RootComponent = DestructibleComponent;
}

#if WITH_EDITOR
bool ADestructibleActor::GetReferencedContentObjects( TArray<UObject*>& Objects ) const
{
	Super::GetReferencedContentObjects(Objects);

	if (DestructibleComponent && DestructibleComponent->GetSkinnedAsset())
	{
		Objects.Add(DestructibleComponent->GetSkinnedAsset());
	}
	return true;
}

void ADestructibleActor::PostLoad()
{
	Super::PostLoad();

	if (DestructibleComponent && bAffectNavigation)
	{
		DestructibleComponent->SetCanEverAffectNavigation(bAffectNavigation);
		bAffectNavigation = false;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

