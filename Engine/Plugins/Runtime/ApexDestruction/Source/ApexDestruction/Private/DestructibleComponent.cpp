// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DestructibleComponent.cpp: UDestructibleComponent methods.
=============================================================================*/

#include "DestructibleComponent.h"
#include "ApexDestructionCustomPayload.h"
#include "GameFramework/DamageType.h"
#include "Async/TaskGraphInterfaces.h"
#include "PhysicsEngine/BodySetup.h"
#include "DestructibleMesh.h"
#include "Engine/SkinnedAsset.h"
#include "ObjectEditorUtils.h"
#include "UObject/UObjectThreadContext.h"
#include "Engine/DamageEvents.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UDestructibleComponent::UDestructibleComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	PrimaryComponentTick.bCanEverTick = false;

	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::Yes;

	static FName CollisionProfileName(TEXT("Destructible"));
	SetCollisionProfileName(CollisionProfileName);

	bAlwaysCreatePhysicsState = true;
	SetActiveFlag(true);
	bMultiBodyOverlap = true;

	LargeChunkThreshold = 25.f;

	SetComponentSpaceTransformsDoubleBuffering(false);
}

#if WITH_EDITORONLY_DATA
void UDestructibleComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if(Ar.IsLoading())
	{
		// Copy our skeletal mesh value to our transient variable, so it appears in slate correctly.
		this->DestructibleMesh_DEPRECATED = GetDestructibleMesh();
	}
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void UDestructibleComponent::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent )
{
	static const FName NAME_DestructibleComponent = FName(TEXT("DestructibleComponent"));
	static const FName NAME_DestructibleMesh = FName(TEXT("DestructibleMesh"));

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != NULL)
	{
		if ((FObjectEditorUtils::GetCategoryFName(PropertyChangedEvent.Property) == NAME_DestructibleComponent)
		&&  (PropertyChangedEvent.Property->GetFName() == NAME_DestructibleMesh))
		{
			// If our transient mesh has changed, update our skeletal mesh.
			SetSkeletalMesh( this->DestructibleMesh_DEPRECATED );
		}
	}
}
#endif // WITH_EDITOR

FBoxSphereBounds UDestructibleComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return Super::CalcBounds(LocalToWorld);
}

bool IsImpactDamageEnabled(const UDestructibleMesh* TheDestructibleMesh, int32 Level)
{
	if(TheDestructibleMesh->DefaultDestructibleParameters.DamageParameters.ImpactDamage == 0.f)
	{
		return false;
	}

	bool bEnableImpactDamage = false;
	const FDestructibleDepthParameters& DepthParams = TheDestructibleMesh->DefaultDestructibleParameters.DepthParameters[Level];
	const EImpactDamageOverride LevelOverride = DepthParams.ImpactDamageOverride; 

	switch(LevelOverride)
	{
		case EImpactDamageOverride::IDO_On:
		{
			return true;
		}

		case EImpactDamageOverride::IDO_Off:
		{
			return false;
		}

		default:
		{
			//return default if we're within the default level
		    return TheDestructibleMesh->DefaultDestructibleParameters.DamageParameters.DefaultImpactDamageDepth >= Level ? TheDestructibleMesh->DefaultDestructibleParameters.DamageParameters.bEnableImpactDamage : false;
		}
	}
}

void UDestructibleComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	// We are handling the physics move below, so don't handle it at higher levels
	Super::OnUpdateTransform(UpdateTransformFlags | EUpdateTransformFlags::SkipPhysicsUpdate, Teleport);

	if (GetSkinnedAsset() == NULL)
	{
		return;
	}

	if (!bPhysicsStateCreated || !!(UpdateTransformFlags & EUpdateTransformFlags::SkipPhysicsUpdate))
	{
		return;
	}

	const FTransform& CurrentLocalToWorld = GetComponentTransform();

#if !(UE_BUILD_SHIPPING)
	if(CurrentLocalToWorld.ContainsNaN())
	{
		logOrEnsureNanError(TEXT("UDestructibleComponent:OnUpdateTransform found NaN in CurrentLocalToWorld: %s"), *CurrentLocalToWorld.ToString());
		return;
	}
#endif

	// warn if it has non-uniform scale
	const FVector& MeshScale3D = CurrentLocalToWorld.GetScale3D();
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if( !MeshScale3D.IsUniform() )
	{
		UE_LOGF(LogPhysics, Log, "UDestructibleComponent::SendPhysicsTransform : Non-uniform scale factor (%ls) can cause physics to mismatch for %ls  SkelMesh: %ls", *MeshScale3D.ToString(), *GetFullName(), GetSkinnedAsset() ? *GetSkinnedAsset()->GetFullName() : TEXT("NULL"));
	}
#endif
}

void UDestructibleComponent::OnCreatePhysicsState()
{
	// to avoid calling PrimitiveComponent, I'm just calling ActorComponent::OnCreatePhysicsState
	// @todo lh - fix me based on the discussion with Bryan G
	UActorComponent::OnCreatePhysicsState();
	bPhysicsStateCreated = true;

	// What we want to do with BodySetup is simply use it to store a PhysicalMaterial, and possibly some other relevant fields.  Set up pointers from the BodyInstance to the BodySetup and this component
	UBodySetup* BodySetup = GetBodySetup();
	BodyInstance.OwnerComponent	= this;
	BodyInstance.BodySetup = BodySetup;
	BodyInstance.InstanceBodyIndex = 0;
}

void UDestructibleComponent::OnDestroyPhysicsState()
{
	Super::OnDestroyPhysicsState();
}

UBodySetup* UDestructibleComponent::GetBodySetup()
{
	if (GetSkinnedAsset() != NULL)
	{
		UDestructibleMesh* TheDestructibleMesh = GetDestructibleMesh();

		if (TheDestructibleMesh != NULL)
		{
			const UDestructibleMesh* TheDestructibleMeshConst = TheDestructibleMesh;
			return TheDestructibleMeshConst->GetBodySetup();
		}
	}

	return NULL;
}

bool UDestructibleComponent::CanEditSimulatePhysics()
{
	// if destructiblemeshcomponent, we will allow it always
	return true;
}

void UDestructibleComponent::AddImpulse( FVector Impulse, FName BoneName /*= NAME_None*/, bool bVelChange /*= false*/ )
{
}

void UDestructibleComponent::AddImpulseAtLocation( FVector Impulse, FVector Position, FName BoneName /*= NAME_None*/ )
{
}

void UDestructibleComponent::AddForce( FVector Force, FName BoneName /*= NAME_None*/, bool bAccelChange /* = false */ )
{
}

void UDestructibleComponent::AddForceAtLocation( FVector Force, FVector Location, FName BoneName /*= NAME_None*/ )
{
}

void UDestructibleComponent::AddForceAtLocationLocal(FVector Force, FVector Location, FName BoneName /*= NAME_None*/)
{
}

void UDestructibleComponent::AddRadialImpulse(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bVelChange)
{
}

void UDestructibleComponent::AddRadialForce(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bAccelChange /* = false */)
{
}

void UDestructibleComponent::ReceiveComponentDamage(float DamageAmount, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser)
{
	UDamageType const* const DamageTypeCDO = DamageEvent.DamageTypeClass ? DamageEvent.DamageTypeClass->GetDefaultObject<UDamageType>() : GetDefault<UDamageType>();
	if (DamageEvent.IsOfType(FPointDamageEvent::ClassID))
	{
		FPointDamageEvent const* const PointDamageEvent = (FPointDamageEvent*)(&DamageEvent);
		ApplyDamage(DamageAmount, PointDamageEvent->HitInfo.ImpactPoint, PointDamageEvent->ShotDirection, DamageTypeCDO->DestructibleImpulse);
	}
	else if (DamageEvent.IsOfType(FRadialDamageEvent::ClassID))
	{
		FRadialDamageEvent const* const RadialDamageEvent = (FRadialDamageEvent*)(&DamageEvent);
		ApplyRadiusDamage(DamageAmount, RadialDamageEvent->Origin, RadialDamageEvent->Params.OuterRadius, DamageTypeCDO->DestructibleImpulse, false);
	}
}

bool UDestructibleComponent::IsFracturedOrInitiallyStatic() const
{
	return false;
}

bool UDestructibleComponent::ExecuteOnPhysicsReadOnly(TFunctionRef<void()> Func) const
{
	return false;
}

bool UDestructibleComponent::ExecuteOnPhysicsReadWrite(TFunctionRef<void()> Func) const
{
	return false;
}

void UDestructibleComponent::RefreshBoneTransforms(FActorComponentTickFunction* TickFunction)
{
}

void UDestructibleComponent::SetDestructibleMesh(class UDestructibleMesh* NewMesh)
{
	Super::SetSkeletalMesh( NewMesh );

#if WITH_EDITORONLY_DATA
	// If the SkeletalMesh has changed, update our transient value too.
	DestructibleMesh_DEPRECATED = GetDestructibleMesh();
#endif // WITH_EDITORONLY_DATA
	
	RecreatePhysicsState();
}

class UDestructibleMesh* UDestructibleComponent::GetDestructibleMesh()
{
	return Cast<UDestructibleMesh>(GetSkinnedAsset());
}

void UDestructibleComponent::SetSkeletalMesh(USkeletalMesh* InSkelMesh, bool bReinitPose)
{
	if(InSkelMesh != NULL && !InSkelMesh->IsA(UDestructibleMesh::StaticClass()))
	{
		// Issue warning and do nothing if this is not actually a UDestructibleMesh
		UE_LOGF(LogPhysics, Log, "UDestructibleComponent::SetSkeletalMesh(): Passed-in USkeletalMesh (%ls) must be a UDestructibleMesh.  SkeletalMesh not set.", *InSkelMesh->GetPathName() );
		return;
	}

	UDestructibleMesh* TheDestructibleMesh = static_cast<UDestructibleMesh*>(InSkelMesh);
	SetDestructibleMesh(TheDestructibleMesh);
}

FTransform UDestructibleComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	FTransform ST = Super::GetSocketTransform(InSocketName, TransformSpace);

	int32 BoneIdx = GetBoneIndex(InSocketName);

	// As bones in a destructible might be scaled to 0 when hidden, we force a scale of 1 if we want the socket transform
	if (BoneIdx > 0 && IsBoneHidden(BoneIdx))
	{
		ST.SetScale3D(FVector(1.0f, 1.0f, 1.0f));
	}

	return ST;
}

void UDestructibleComponent::SetChunkVisible( int32 ChunkIndex, bool bInVisible )
{
}


void UDestructibleComponent::SetChunksWorldTM(const TArray<FUpdateChunksInfo>& UpdateInfos)
{
	const FQuat InvRotation = GetComponentTransform().GetRotation().Inverse();

	for (const FUpdateChunksInfo& UpdateInfo : UpdateInfos)
	{
		// Bone 0 is a dummy root bone
		const int32 BoneIndex = ChunkIdxToBoneIdx(UpdateInfo.ChunkIndex);
		const FVector WorldTranslation	= UpdateInfo.WorldTM.GetLocation();
		const FQuat WorldRotation		= UpdateInfo.WorldTM.GetRotation();

		const FQuat BoneRotation = InvRotation*WorldRotation;
		const FVector BoneTranslation = InvRotation.RotateVector(WorldTranslation - GetComponentTransform().GetTranslation()) / GetComponentTransform().GetScale3D();

		GetEditableComponentSpaceTransforms()[BoneIndex] = FTransform(BoneRotation, BoneTranslation);
	}

	bNeedToFlipSpaceBaseBuffers = true;

	// Mark the transform as dirty, so the bounds are updated and sent to the render thread
	MarkRenderTransformDirty();

	// New bone positions need to be sent to render thread
	MarkRenderDynamicDataDirty();

	//Update bone visibilty and flip the editable space base buffer
	FinalizeBoneTransform();
}

void UDestructibleComponent::SetChunkWorldRT( int32 ChunkIndex, const FQuat& WorldRotation, const FVector& WorldTranslation )
{
	// Bone 0 is a dummy root bone
	const int32 BoneIndex = ChunkIdxToBoneIdx(ChunkIndex);

	// Mark the transform as dirty, so the bounds are updated and sent to the render thread
	MarkRenderTransformDirty();

	// New bone positions need to be sent to render thread
	MarkRenderDynamicDataDirty();

#if 0
	// Scale is already applied to the GetComponentTransform() transform, and is carried into the bones _locally_.
	// So there is no need to set scale in the bone local transforms
	const FTransform WorldRT(WorldRotation, WorldTranslation, GetComponentTransform().GetScale3D());
	SpaceBases(BoneIndex) = WorldRT*GetComponentTransform().Inverse();
#elif 1
	// More optimal form of the above
	const FQuat BoneRotation = GetComponentTransform().GetRotation().Inverse()*WorldRotation;
	const FVector BoneTranslation = GetComponentTransform().GetRotation().Inverse().RotateVector(WorldTranslation - GetComponentTransform().GetTranslation())/GetComponentTransform().GetScale3D();
	GetEditableComponentSpaceTransforms()[BoneIndex] = FTransform(BoneRotation, BoneTranslation);
#endif
}

void UDestructibleComponent::ApplyDamage(float DamageAmount, const FVector& HitLocation, const FVector& ImpulseDir, float ImpulseStrength)
{
}

void UDestructibleComponent::ApplyRadiusDamage(float BaseDamage, const FVector& HurtOrigin, float DamageRadius, float ImpulseStrength, bool bFullDamage)
{
}

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Destructible shapes exported"),STAT_Navigation_DestructiblesShapesExported,STATGROUP_Navigation );

bool UDestructibleComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	bool bExportFromBodySetup = true;

	// we don't want a regular geometry export
	return bExportFromBodySetup;
}

void UDestructibleComponent::Activate( bool bReset/*=false*/ )
{
	if (bReset || ShouldActivate()==true)
	{
		SetActiveFlag(true);
	}
}

void UDestructibleComponent::Deactivate()
{
	if (ShouldActivate()==false)
	{
		SetActiveFlag(false);
	}
}

void UDestructibleComponent::BeginPlay()
{
	Super::BeginPlay();

#if WITH_DESTRUCTIBLE_DEPRECATION
	// Deprecation message as of UE5.1
	UE_LOGF(LogDestructible, Warning, "(%ls): DestructibleComponent is deprecated. Destruction is now supported by UGeometryCollection. Please update your assets before upgrading to the next release.", *GetPathName());
#endif
}

void UDestructibleComponent::SetCollisionResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse)
{
}

void UDestructibleComponent::SetCollisionResponseToAllChannels(ECollisionResponse NewResponse)
{
}

void UDestructibleComponent::SetCollisionResponseToChannels(const FCollisionResponseContainer& NewReponses)
{
}

bool UDestructibleComponent::ShouldUpdateTransform(bool bLODHasChanged) const
{
	// We do not want to update bone transforms before physics has finished
	return false;
}

bool UDestructibleComponent::LineTraceComponent( FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params )
{
	return false;
}

bool UDestructibleComponent::SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FCollisionShape &CollisionShape, bool bTraceComplex/*=false*/)
{
	return false;
}

void UDestructibleComponent::WakeRigidBody(FName BoneName /* = NAME_None */)
{
}

void UDestructibleComponent::SetSimulatePhysics(bool bSimulate)
{
}

void UDestructibleComponent::SetEnableGravity(bool bGravityEnabled)
{
	Super::SetEnableGravity(bGravityEnabled);
}

FBodyInstance* UDestructibleComponent::GetBodyInstance( FName BoneName /*= NAME_None*/, bool, int) const
{
	return (FBodyInstance*)&BodyInstance;
}

bool UDestructibleComponent::IsAnySimulatingPhysics() const 
{
	return !!BodyInstance.bSimulatePhysics;
}

void UDestructibleComponent::OnActorEnableCollisionChanged()
{
	ECollisionEnabled::Type NewCollisionType = GetBodyInstance()->GetCollisionEnabled();
	SetCollisionEnabled(NewCollisionType);
}

void UDestructibleComponent::SetCollisionEnabled(ECollisionEnabled::Type NewType)
{
}

void UDestructibleComponent::SetCollisionProfileName(FName InCollisionProfileName, bool bUpdateOverlaps)
{
    FBodyInstance* LocalInstance = GetBodyInstance();
    if (!LocalInstance)
    {
        return;
    }

	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	if (ThreadContext.ConstructedObject == this)
	{
		// If we are in our constructor, defer setup until PostInitProperties as derived classes
		// may call SetCollisionProfileName more than once.
		LocalInstance->SetCollisionProfileNameDeferred(InCollisionProfileName);
	}
	else
	{
		ECollisionEnabled::Type OldCollisionEnabled = LocalInstance->GetCollisionEnabled();
		LocalInstance->SetCollisionProfileName(InCollisionProfileName);

		ECollisionEnabled::Type NewCollisionEnabled = LocalInstance->GetCollisionEnabled();

		if (OldCollisionEnabled != NewCollisionEnabled)
		{
			EnsurePhysicsStateCreated();
		}
		OnComponentCollisionSettingsChanged(bUpdateOverlaps);
	}
}

void UDestructibleComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* Material)
{
	// Mesh component handles render side materials
	Super::SetMaterial(ElementIndex, Material);

	// Update physical properties of the chunks in the mesh if the body instance is valid
	FBodyInstance* BodyInst = GetBodyInstance();
	if (BodyInst)
	{
		BodyInst->UpdatePhysicalMaterials();
	}
	
	// Update physical properties for individual bone instances as well
	if (GetSkinnedAsset())
	{
		int32 NumBones = GetSkinnedAsset()->GetRefSkeleton().GetRawBoneNum();
		for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
		{
			FName BoneName = GetSkinnedAsset()->GetRefSkeleton().GetBoneName(BoneIdx);
			FBodyInstance* Instance = GetBodyInstance(BoneName);
			if (Instance)
			{
				Instance->UpdatePhysicalMaterials();
			}
		}
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
