// Copyright Epic Games, Inc. All Rights Reserved.

#include "DestructibleFractureSettings.h"

//////////////////////////////////////////////////////////////////////////

UDestructibleFractureSettings::~UDestructibleFractureSettings()
{
}

// Local utilities

#if WITH_EDITOR	//	Fracture code is only needed in editor

// Why doesn't TArray have this function?
template<typename ElementType>
inline void Resize(TArray<ElementType>& Array, const ElementType& Item, uint32 Size)
{
	const uint32 OldSize = Array.Num();
	if (Size < OldSize)
	{
		Array.RemoveAt(Size, OldSize-Size);
	}
	else
	if (Size > OldSize)
	{
		Array.Reserve(Size);
		for (uint32 Index = OldSize; Index < Size; ++Index)
		{
			Array.Add(Item);
		}
	}
}
#endif // #if WITH_EDITOR	//	Fracture code is only needed in editor

//////////////////////////////////////////////////////////////////////////
// UDestructibleFractureSettings
//////////////////////////////////////////////////////////////////////////
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDestructibleFractureSettings::UDestructibleFractureSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CellSiteCount = 25;
	OriginalSubmeshCount = 0;
}

UDestructibleFractureSettings::UDestructibleFractureSettings(FVTableHelper& Helper)
	: Super(Helper)
{

}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UDestructibleFractureSettings::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR	//	Fracture code is only needed in editor

	if (Ar.IsLoading())
	{
		// Buffer for the NxDestructibleAsset(Authoring)
		TArray<uint8> Buffer;
		uint32 Size;
		Ar << Size;
		if (Size > 0)
		{
			// Size is non-zero, so a binary blob follows
			Buffer.AddUninitialized(Size);
			Ar.Serialize(Buffer.GetData(), Size);
		}
	}
	else if (Ar.IsSaving())
	{
		uint32 size=0;
		Ar << size;
	}

#endif // #if WITH_EDITOR	//	Fracture code is only needed in editor
}

//IMPLEMENT_FSTRUCTUREDARCHIVE_SERIALIZER(UDestructibleFractureSettings);

void UDestructibleFractureSettings::PostInitProperties()
{
	Super::PostInitProperties();
}

void UDestructibleFractureSettings::BeginDestroy()
{
	Super::BeginDestroy();
}
