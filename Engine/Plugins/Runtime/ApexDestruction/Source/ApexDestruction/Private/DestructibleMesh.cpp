// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DestructibleMesh.cpp: UDestructibleMesh methods.
=============================================================================*/

#include "DestructibleMesh.h"
#include "Async/TaskGraphInterfaces.h"
#include "DestructibleFractureSettings.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Engine/StaticMesh.h"

#if WITH_EDITOR
#include "EditorFramework/AssetImportData.h"
#endif

DEFINE_LOG_CATEGORY(LogDestructible)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDestructibleMesh::UDestructibleMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UDestructibleMesh::PostLoad()
{
	Super::PostLoad();

#if WITH_DESTRUCTIBLE_DEPRECATION
	// Deprecation message as of UE5.1
	UE_LOGF(LogDestructible, Warning, "During PostLoad for Mesh %ls: DestructibleMesh is deprecated. Destruction is now supported by UGeometryCollection. Please update your assets before upgrading to the next release.", *GetPathName());
#endif

	// BodySetup is used for uniform lookup of PhysicalMaterials.
	CreateBodySetup();
}

#if WITH_EDITOR
void UDestructibleMesh::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	CreateBodySetup();
}
#endif

float ImpactResistanceToAPEX(bool bCustomResistance, float ImpactResistance)
{
	return bCustomResistance ? ImpactResistance : 0.f;
}


void APEXToImpactResistance(bool& bCustomImpactResistance, float& ImpactResistance)
{
	//apex interprets 0 as disabled, but we want custom flag
	bCustomImpactResistance = ImpactResistance != 0.f;
	if (ImpactResistance == 0.f)
	{
		ImpactResistance = 1.f;
	}
}

int32 DefaultImpactDamageDepthToAPEX(bool bEnableImpactDamage, int32 DefaultImpactDamageDepth)
{
	return bEnableImpactDamage ? DefaultImpactDamageDepth : -1.f;
}

void APEXToDefaultImpactDamageDepth(bool& bEnableImpactDamage, int32& DefaultImpactDamageDepth)
{
	//apex interprets -1 as disabled, but we want custom flag
	bEnableImpactDamage = DefaultImpactDamageDepth != -1;
	DefaultImpactDamageDepth = 1;
}

int32 DebrisDepthToAPEX(bool bEnableDebris, int32 DebrisDepth)
{
	return bEnableDebris ? DebrisDepth : -1;
}

void APEXToDebrisDepth(bool& bEnableDebris, int32& DebrisDepth)
{
	bEnableDebris = DebrisDepth != -1;
	if (DebrisDepth == -1)
	{
		DebrisDepth = 0;
	}
}

void UDestructibleMesh::Serialize(FArchive& Ar)
{
	Super::Serialize( Ar );

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if( Ar.IsLoading() )
	{
		// Deserializing the name of the NxDestructibleAsset
		TArray<uint8> NameBuffer;
		uint32 NameBufferSize;
		Ar << NameBufferSize;
		NameBuffer.AddUninitialized( NameBufferSize );
		Ar.Serialize( NameBuffer.GetData(), NameBufferSize );

		// Buffer for the NxDestructibleAsset
		TArray<uint8> Buffer;
		uint32 Size;
		Ar << Size;
		if( Size > 0 )
		{
			// Size is non-zero, so a binary blob follows
			// Discard apex serialized buffer as no longer supported
			Buffer.AddUninitialized( Size );
			Ar.Serialize( Buffer.GetData(), Size );
		}
		if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::CacheDestructibleOverlaps)
		{
			Ar << Size;
			if( Size > 0 )
			{
				// Here comes the collision data cache
				Buffer.Reset(Size);
				Buffer.AddZeroed(Size);

				Ar.Serialize( Buffer.GetData(), Size );
			}
		}
	}
	else if ( Ar.IsSaving() )
	{
		const char* Name = "NO_APEX";
		// Serialize the name
		uint32 NameBufferSize = FCStringAnsi::Strlen( Name )+1;
		Ar << NameBufferSize;
		Ar.Serialize( (void*)Name, NameBufferSize );

		uint32 size=0;
		Ar << size; // Buffer for the NxDestructibleAsset
		Ar << size; // collision data cache
	}

	if (Ar.UEVer() < VER_UE4_CLEAN_DESTRUCTIBLE_SETTINGS)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		APEXToImpactResistance(DefaultDestructibleParameters.DamageParameters.bCustomImpactResistance, DefaultDestructibleParameters.DamageParameters.ImpactResistance);
		APEXToDefaultImpactDamageDepth(DefaultDestructibleParameters.DamageParameters.bEnableImpactDamage, DefaultDestructibleParameters.DamageParameters.DefaultImpactDamageDepth);
		APEXToDebrisDepth(DefaultDestructibleParameters.SpecialHierarchyDepths.bEnableDebris, DefaultDestructibleParameters.SpecialHierarchyDepths.DebrisDepth);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UDestructibleMesh::FinishDestroy()
{
	Super::FinishDestroy();
}

void UDestructibleMesh::LoadDefaultDestructibleParametersFromApexAsset()
{

}

void UDestructibleMesh::CreateFractureSettings()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
	if (FractureSettings == NULL)
	{
		FractureSettings = NewObject<UDestructibleFractureSettings>(this);
		check(FractureSettings);
	}
#endif	// WITH_EDITORONLY_DATA
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UDestructibleMesh::BuildFractureSettingsFromStaticMesh(UStaticMesh* StaticMesh)
{
	return false;
}

APEXDESTRUCTION_API bool UDestructibleMesh::BuildFromStaticMesh( UStaticMesh& StaticMesh )
{
#if WITH_EDITOR
	PreEditChange(NULL);

	// Import the StaticMesh
	if (!BuildFractureSettingsFromStaticMesh(&StaticMesh))
	{
		return false;
	}

	SourceStaticMesh = &StaticMesh;

	SourceSMImportTimestamp = FDateTime::MinValue();
	if ( StaticMesh.GetAssetImportData() && StaticMesh.GetAssetImportData()->SourceData.SourceFiles.Num() == 1)
	{
		SourceSMImportTimestamp = StaticMesh.GetAssetImportData()->SourceData.SourceFiles[0].Timestamp;
	}

	PostEditChange();
	MarkPackageDirty();
#endif // WITH_EDITOR
	return true;
}

APEXDESTRUCTION_API bool UDestructibleMesh::SetupChunksFromStaticMeshes( const TArray<UStaticMesh*>& ChunkMeshes )
{
#if WITH_EDITOR
	if (SourceStaticMesh == NULL)
	{
		UE_LOGF(LogDestructible, Warning, "Unable to import FBX as level 1 chunks if the DM was not created from a static mesh.");
		return false;
	}

	PreEditChange(NULL);

	FractureChunkMeshes.Empty(ChunkMeshes.Num());
	FractureChunkMeshes.Append(ChunkMeshes);
	
	// Import the StaticMesh
	if (!BuildFractureSettingsFromStaticMesh(SourceStaticMesh))
	{
		return false;
	}

	// Clear the fracture chunk meshes again
	FractureChunkMeshes.Empty();

	PostEditChange();
	MarkPackageDirty();
#endif // WITH_EDITOR
	return true;
}
