// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "DestructibleFractureSettings.h"

#if WITH_EDITOR

class FSkeletalMeshImportData;
class UDestructibleMesh;

#endif