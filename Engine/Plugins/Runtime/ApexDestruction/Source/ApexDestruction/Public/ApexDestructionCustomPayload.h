// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CustomPhysXPayload.h"

struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FUpdateChunksInfo
{
	int32 ChunkIndex;
	FTransform WorldTM;

	FUpdateChunksInfo(int32 InChunkIndex, const FTransform & InWorldTM) : ChunkIndex(InChunkIndex), WorldTM(InWorldTM)
	{}
};
