// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PhysXPublic.h"

namespace nvidia
{

namespace apex
{
	class ModuleDestructible;
	struct DamageEventReportData;
}
}

extern nvidia::apex::ModuleDestructible* GApexModuleDestructible;