// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** This class represents an APEX Destructible Actor. */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "DestructibleActor.generated.h"

class UDestructibleComponent;

/** Delegate for notification when fracture occurs */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FActorFractureSignature, const FVector&, HitPoint, const FVector&, HitDirection);

class UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") ADestructibleActor;
UCLASS(hideCategories=(Input), showCategories=("Input|MouseInput", "Input|TouchInput"), ComponentWrapperClass)
class APEXDESTRUCTION_API ADestructibleActor : public AActor
{
	GENERATED_UCLASS_BODY()

	/**
	 * The component which holds the skinned mesh and physics data for this actor.
	 */
private:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Destruction, meta = (ExposeFunctionCategories = "Destruction,Components|Destructible", AllowPrivateAccess = "true"))
	TObjectPtr<UDestructibleComponent> DestructibleComponent;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
public:

	UPROPERTY(BlueprintAssignable, Category = "Components|Destructible")
	FActorFractureSignature OnActorFracture;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(4.24, "This property will be removed in future versions. Please use bCanEverAffectNavigation in DestructionComponent.")
	UPROPERTY(config, BlueprintReadWrite, Category = Navigation, meta = (DeprecationMessage = "Setting the value from Blueprint script does nothing. Please use bCanEverAffectNavigation in DestructionComponent."))
	uint32 bAffectNavigation : 1;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	//~ Begin AActor Interface.
	virtual bool GetReferencedContentObjects( TArray<UObject*>& Objects ) const override;
	virtual void PostLoad() override;
	//~ End AActor Interface.
#endif // WITH_EDITOR

public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Returns DestructibleComponent subobject **/
	UDestructibleComponent* GetDestructibleComponent() const { return DestructibleComponent; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};



