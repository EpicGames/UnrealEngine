// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "EngineDefines.h"
#include "PhysxUserData.h"
#include "ApexDestructionCustomPayload.h"
#include "Components/SkinnedMeshComponent.h"
#include "DestructibleInterface.h"
#include "DestructibleComponent.generated.h"

class AController;
class USkeletalMesh;
struct FBodyInstance;
struct FCollisionShape;
struct FNavigableGeometryExport;

/** Delegate for notification when fracture occurs */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FComponentFractureSignature, const FVector &, HitPoint, const FVector &, HitDirection);

/**
 *	This component holds the physics data for a DestructibleActor.
 *
 *	The USkeletalMesh pointer in the base class (SkinnedMeshComponent) MUST be a DestructibleMesh
 */

class UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") UDestructibleComponent;
UCLASS(ClassGroup = Physics, hidecategories = (Object, Mesh, "Mesh|SkeletalAsset", "Components|SkinnedMesh", Mirroring, Activation, "Components|Activation"), config = Engine, editinlinenew, meta = (BlueprintSpawnableComponent))
class APEXDESTRUCTION_API UDestructibleComponent : public USkinnedMeshComponent, public IDestructibleInterface
{
	GENERATED_UCLASS_BODY()

	/** If set, use this actor's fracture effects instead of the asset's fracture effects. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DestructibleComponent)
	uint32 bFractureEffectOverride:1;

	/** Fracture effects for each fracture level. Used only if Fracture Effect Override is set. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, editfixedsize, Category=DestructibleComponent)
	TArray<struct FFractureEffect> FractureEffects;

	/**
	 *	Enable "hard sleeping" for destruction-generated PxActors.  This means that they turn kinematic
	 *	when they sleep, but can be made dynamic again by application of enough damage.
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=DestructibleComponent)
	bool bEnableHardSleeping;

	/**
	 * The minimum size required to treat chunks as Large.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = DestructibleComponent)
	float LargeChunkThreshold;

#if WITH_EDITORONLY_DATA
	/** Provide a blueprint interface for setting the destructible mesh */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(Transient, BlueprintSetter=SetDestructibleMesh, BlueprintGetter=GetDestructibleMesh, Category = "Components|Destructible", meta = (DeprecatedProperty, DeprecatedMessage = "Destructible mesh is now deprected, use UGeometryCollection instead"))
	TObjectPtr<class UDestructibleMesh> DestructibleMesh_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	//~ Begin UObject Interface.
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
#if WITH_EDITORONLY_DATA
	virtual void Serialize(FArchive& Ar) override;
#endif
	//~ End UObject Interface.

	// Take damage
	UFUNCTION(BlueprintCallable, Category="Components|Destructible")
	virtual void ApplyDamage(float DamageAmount, const FVector& HitLocation, const FVector& ImpulseDir, float ImpulseStrength) override;

	// Take radius damage
	UFUNCTION(BlueprintCallable, Category="Components|Destructible")
	virtual void ApplyRadiusDamage(float BaseDamage, const FVector& HurtOrigin, float DamageRadius, float ImpulseStrength, bool bFullDamage) override;

	UE_DEPRECATED("5.1", "Destructible mesh is now deprected, use UGeometryCollection instead")
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category="Components|Destructible", meta=(DeprecatedFunction, DeprecationMessage="Destructible mesh is now deprected, use UGeometryCollection instead"))
	void SetDestructibleMesh(class UDestructibleMesh* NewMesh);

	UE_DEPRECATED("5.1", "Destructible mesh is now deprected, use UGeometryCollection instead")
	UFUNCTION(BlueprintCallable, BlueprintGetter, Category="Components|Destructible", meta = (DeprecatedFunction, DeprecationMessage = "Destructible mesh is now deprected, use UGeometryCollection instead"))
	class UDestructibleMesh* GetDestructibleMesh();

	/** Called when a component is touched */
	UPROPERTY(BlueprintAssignable, Category = "Components|Destructible")
	FComponentFractureSignature OnComponentFracture;

public:

	//~ Begin USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport = ETeleportType::None) override;
	virtual void Activate(bool bReset=false) override;
	virtual void Deactivate() override;
	//~ End USceneComponent Interface.

	//~ Begin UActorComponent Interface.
	virtual void BeginPlay() override;
protected:
	virtual void OnCreatePhysicsState() override;
	virtual void OnDestroyPhysicsState() override;
public:
	virtual class UBodySetup* GetBodySetup() override;
	//~ End UActorComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
	virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = INDEX_NONE) const override;
	virtual bool CanEditSimulatePhysics() override;
	virtual bool IsAnySimulatingPhysics() const override;

	virtual void AddImpulse(FVector Impulse, FName BoneName = NAME_None, bool bVelChange = false) override;
	virtual void AddImpulseAtLocation(FVector Impulse, FVector Position, FName BoneName = NAME_None) override;
	virtual void AddForce(FVector Force, FName BoneName = NAME_None, bool bAccelChange = false) override;
	virtual void AddForceAtLocation(FVector Force, FVector Location, FName BoneName = NAME_None) override;
	virtual void AddForceAtLocationLocal(FVector Force, FVector Location, FName BoneName = NAME_None) override;
	virtual void AddRadialImpulse(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bVelChange=false) override;
	virtual void AddRadialForce(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bAccelChange = false) override;
	virtual void ReceiveComponentDamage(float DamageAmount, FDamageEvent const& DamageEvent, AController* EventInstigator, AActor* DamageCauser) override;

	virtual bool LineTraceComponent( FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params ) override;
    virtual bool SweepComponent( FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FCollisionShape& CollisionShape, bool bTraceComplex=false) override;
	virtual void SetEnableGravity(bool bGravityEnabled) override;

	virtual void WakeRigidBody(FName BoneName /* = NAME_None */) override;
	virtual void SetSimulatePhysics(bool bSimulate) override;

	virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	virtual void SetCollisionEnabled(ECollisionEnabled::Type NewType) override;
    virtual void SetCollisionProfileName(FName InCollisionProfileName, bool bUpdateOverlaps=true) override;
	virtual void OnActorEnableCollisionChanged() override;

	virtual void SetCollisionResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse) override;
	virtual void SetCollisionResponseToAllChannels(ECollisionResponse NewResponse) override;
	virtual void SetCollisionResponseToChannels(const FCollisionResponseContainer& NewReponses) override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin SkinnedMeshComponent Interface.
	virtual bool ShouldUpdateTransform(bool bLODHasChanged) const override;
	virtual void RefreshBoneTransforms(FActorComponentTickFunction* TickFunction = NULL) override;
	virtual void SetSkeletalMesh(USkeletalMesh* InSkelMesh, bool bReinitPose = true) override;
	virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	//~ End SkinnedMeshComponent Interface.


	//~ Begin DestructibleComponent Interface.
	/** This method makes a chunk (fractured piece) visible or invisible.
	 *
	 * @param ChunkIndex - Which chunk to affect.  ChunkIndex must lie in the range: 0 <= ChunkIndex < ((DestructibleMesh*)USkeletalMesh)->ApexDestructibleAsset->chunkCount().
	 * @param bVisible - If true, the chunk will be made visible.  Otherwise, the chunk is made invisible.
	 */
	void SetChunkVisible( int32 ChunkIndex, bool bInVisible );


	/** This method sets a chunk's (fractured piece's) world rotation and translation.
	 *
	 * @param ChunkIndex - Which chunk to affect.  ChunkIndex must lie in the range: 0 <= ChunkIndex < ((DestructibleMesh*)USkeletalMesh)->ApexDestructibleAsset->chunkCount().
	 * @param WorldRotation - The orientation to give to the chunk in world space, represented as a quaternion.
	 * @param WorldRotation - The world space position to give to the chunk.
	 */
	void SetChunkWorldRT( int32 ChunkIndex, const FQuat& WorldRotation, const FVector& WorldTranslation );

	//~ End DestructibleComponent Interface.

	virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;

	FORCEINLINE static int32 ChunkIdxToBoneIdx(int32 ChunkIdx) { return ChunkIdx + 1; }
	FORCEINLINE static int32 BoneIdxToChunkIdx(int32 BoneIdx) { return FMath::Max(BoneIdx - 1, 0); }
private:

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void SetChunksWorldTM(const TArray<FUpdateChunksInfo>& UpdateInfos);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool IsFracturedOrInitiallyStatic() const;

	/** Obtains the appropriate PhysX scene lock for READING and executes the passed in lambda. */
	bool ExecuteOnPhysicsReadOnly(TFunctionRef<void()> Func) const;

	/** Obtains the appropriate PhysX scene lock for WRITING and executes the passed in lambda. */
	bool ExecuteOnPhysicsReadWrite(TFunctionRef<void()> Func) const;

	/** Collision response used for chunks */
	FCollisionResponse LargeChunkCollisionResponse;
	FCollisionResponse SmallChunkCollisionResponse;

private:
	/** Cached values for computing contact offsets */
	float ContactOffsetFactor; 
	float MinContactOffset;
	float MaxContactOffset;

	friend struct FApexDestructionSyncActors;
};



