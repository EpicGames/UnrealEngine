// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EngineDefines.h"
#include "DestructibleFractureSettings.generated.h"

class UMaterialInterface;

/** 
 * Options for APEX asset import.
 **/
namespace EDestructibleImportOptions
{
	enum Type
	{
		// Just imports the APEX asset
		None				= 0,
		// Preserves settings in DestructibleMesh 
		PreserveSettings	= 1<<0,
	};
};


/** Parameters to describe the application of U,V coordinates on a particular slice within a destructible. */
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FFractureMaterial;
USTRUCT()
struct FFractureMaterial
{
	GENERATED_USTRUCT_BODY()

	/**
	 * The UV scale (geometric distance/unit texture distance) for interior materials.
	 * Default = (100.0f,100.0f).
	 */
	UPROPERTY(EditAnywhere, Category=FractureMaterial)
	FVector2D	UVScale;

	/**
	 * A UV origin offset for interior materials.
	 * Default = (0.0f,0.0f).
	 */
	UPROPERTY(EditAnywhere, Category=FractureMaterial)
	FVector2D	UVOffset;

	/**
	 * Object-space vector specifying surface tangent direction.  If this vector is (0.0f,0.0f,0.0f), then an arbitrary direction will be chosen.
	 * Default = (0.0f,0.0f,0.0f).
	 */
	UPROPERTY(EditAnywhere, Category=FractureMaterial)
	FVector		Tangent;

	/**
	 * Angle from tangent direction for the u coordinate axis.
	 * Default = 0.0f.
	 */
	UPROPERTY(EditAnywhere, Category=FractureMaterial)
	float		UAngle;

	/**
	 * The element index to use for the newly-created triangles.
	 * If a negative index is given, a new element will be created for interior triangles.
	 * Default = -1
	 */
	UPROPERTY(EditAnywhere, Category=FractureMaterial)
	int32		InteriorElementIndex;

	FFractureMaterial()
		: UVScale(100.0f, 100.0f)
		, UVOffset(0.0f, 0.0f)
		, Tangent(0.0f, 0.0f, 0.0f)
		, UAngle(0.0f)
		, InteriorElementIndex(-1)
	{}
};


/** Per-chunk authoring data. */
USTRUCT()
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleChunkParameters
{
	GENERATED_USTRUCT_BODY()

	/**
		Defines the chunk to be environmentally supported, if the appropriate NxDestructibleParametersFlag flags
		are set in NxDestructibleParameters.
		Default = false.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleChunkParameters)
	bool bIsSupportChunk;

	/**
		Defines the chunk to be unfractureable.  If this is true, then none of its children will be fractureable.
		Default = false.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleChunkParameters)
	bool bDoNotFracture;

	/**
		Defines the chunk to be undamageable.  This means this chunk will not fracture, but its children might.
		Default = false.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleChunkParameters)
	bool bDoNotDamage;

	/**
		Defines the chunk to be uncrumbleable.  This means this chunk will not be broken down into fluid mesh particles
		no matter how much damage it takes.  Note: this only applies to chunks with no children.  For a chunk with
		children, then:
		1) The chunk may be broken down into its children, and then its children may be crumbled, if the doNotCrumble flag
		is not set on them.
		2) If the Destructible module's chunk depth offset LOD may be set such that this chunk effectively has no children.
		In this case, the doNotCrumble flag will apply to it.
		Default = false.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleChunkParameters)
	bool bDoNotCrumble;

	FDestructibleChunkParameters()
		: bIsSupportChunk(false)
		, bDoNotFracture(false)
		, bDoNotDamage(false)
		, bDoNotCrumble(false)
	{}
};


/** Information to create an NxDestructibleAsset */
UCLASS(MinimalAPI)
class UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") UDestructibleFractureSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/**  Destructor. Defined to avoid compilation errors on clang where the UHT generated CPP file 
	 * 	ends up instantiating destructors for member variables without full type information
	 */
	virtual ~UDestructibleFractureSettings();

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	UDestructibleFractureSettings(FVTableHelper& Helper);

	/** The number of voronoi cell sites. */
	UPROPERTY(EditAnywhere, Category = Voronoi, meta = (ClampMin = "1", UIMin = "1"))
	int32									CellSiteCount;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Stored interior material data.  Just need one as we only support Voronoi splitting. */
	UPROPERTY(EditAnywhere, transient, Category=General)
	FFractureMaterial						FractureMaterialDesc;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Random seed for reproducibility */
	UPROPERTY(EditAnywhere, Category=General)
	int32									RandomSeed;

	/** Stored Voronoi sites */
	UPROPERTY()
	TArray<FVector>							VoronoiSites;

	/** The mesh's original number of submeshes.  APEX needs to store this in the authoring. */
	UPROPERTY()
	int32									OriginalSubmeshCount;

	/** APEX references materials by name, but we'll bypass that mechanism and use of UE materials instead. */
	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>>				Materials;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Per-chunk authoring parameters, which should be made writable when a chunk selection GUI is in place. */
	UPROPERTY()
	TArray<FDestructibleChunkParameters>	ChunkParameters;
PRAGMA_ENABLE_DEPRECATION_WARNINGS	

	//~ Begin UObject Interface.
	virtual void PostInitProperties() override;
	virtual void BeginDestroy() override;
	virtual void Serialize(FArchive& Ar) override;
	//~ End  UObject Interface
};
