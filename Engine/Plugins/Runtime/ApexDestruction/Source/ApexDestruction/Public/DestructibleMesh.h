// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/CoreMiscDefines.h"
#include "DestructibleMesh.generated.h"

class UPhysicalMaterial;
class UStaticMesh;

#ifndef WITH_DESTRUCTIBLE_DEPRECATION
#define WITH_DESTRUCTIBLE_DEPRECATION 0
#endif

/**
	Chunks up to the depth DefaultImpactDamageDepth will take impact damage, unless IDO_On or IDO_Off is chosen.
*/
UENUM()
enum EImpactDamageOverride : int
{
	IDO_None,
	IDO_On,
	IDO_Off,
	IDO_MAX,
};


/** Properties that may be set for all chunks at a particular depth in the fracture hierarchy. */
USTRUCT()
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleDepthParameters
{
	GENERATED_USTRUCT_BODY()

	/** Chunks up to the depth DefaultImpactDamageDepth will take impact damage, unless one of the override options (see EImpactDamageOverride) is chosen. */
	UPROPERTY(EditAnywhere, Category=DestructibleDepthParameters)
	TEnumAsByte<enum EImpactDamageOverride> ImpactDamageOverride;


	FDestructibleDepthParameters()
		: ImpactDamageOverride(IDO_None)
	{}
};


/** Flags that apply to a destructible actor. */
USTRUCT()
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleParametersFlag
{
	GENERATED_USTRUCT_BODY()

	/**
		If set, chunks will "remember" damage applied to them, so that many applications of a damage amount
		below damageThreshold will eventually fracture the chunk.  If not set, a single application of
		damage must exceed damageThreshold in order to fracture the chunk.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bAccumulateDamage:1;

	/**
		If set, then chunks which are tagged as "support" chunks (via NxDestructibleChunkDesc::isSupportChunk)
		will have environmental support in static destructibles.

		Note: if both bAssetDefinedSupport and bWorldSupport are set, then chunks must be tagged as
		"support" chunks AND overlap the Scene's static geometry in order to be environmentally supported.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bAssetDefinedSupport:1;

	/**
		If set, then chunks which overlap the Scene's static geometry will have environmental support in
		static destructibles.

		Note: if both bAssetDefinedSupport and bWorldSupport are set, then chunks must be tagged as
		"support" chunks AND overlap the Scene's static geometry in order to be environmentally supported.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bWorldSupport:1;

	/**
		Whether or not chunks at or deeper than the "debris" depth (see NxDestructibleParameters::debrisDepth)
		will time out.  The lifetime is a value between NxDestructibleParameters::debrisLifetimeMin and
		NxDestructibleParameters::debrisLifetimeMax, based upon the destructible module's LOD setting.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bDebrisTimeout:1;

	/**
		Whether or not chunks at or deeper than the "debris" depth (see NxDestructibleParameters::debrisDepth)
		will be removed if they separate too far from their origins.  The maxSeparation is a value between
		NxDestructibleParameters::debrisMaxSeparationMin and NxDestructibleParameters::debrisMaxSeparationMax,
		based upon the destructible module's LOD setting.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bDebrisMaxSeparation:1;

	/**
		If set, the smallest chunks may be further broken down, either by fluid crumbles (if a crumble particle
		system is specified in the NxDestructibleActorDesc), or by simply removing the chunk if no crumble
		particle system is specified.  Note: the "smallest chunks" are normally defined to be the deepest level
		of the fracture hierarchy.  However, they may be taken from higher levels of the hierarchy if
		NxModuleDestructible::setMaxChunkDepthOffset is called with a non-zero value.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bCrumbleSmallestChunks:1;

	/**
		If set, the NxDestructibleActor::rayCast function will search within the nearest visible chunk hit
		for collisions with child chunks.  This is used to get a better raycast position and normal, in
		case the parent collision volume does not tightly fit the graphics mesh.  The returned chunk index
		will always be that of the visible parent that is intersected, however.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bAccurateRaycasts:1;

	/**
		If set, the ValidBounds field of NxDestructibleParameters will be used.  These bounds are translated
		(but not scaled or rotated) to the origin of the destructible actor.  If a chunk or chunk island moves
		outside of those bounds, it is destroyed.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bUseValidBounds:1;

	/**
		If initially static, the destructible will become part of an extended support structure if it is
		in contact with another static destructible that also has this flag set.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleParametersFlag)
	uint32 bFormExtendedStructures:1;


	FDestructibleParametersFlag()
		: bAccumulateDamage(false)
		, bAssetDefinedSupport(false)
		, bWorldSupport(false)
		, bDebrisTimeout(false)
		, bDebrisMaxSeparation(false)
		, bCrumbleSmallestChunks(false)
		, bAccurateRaycasts(false)
		, bUseValidBounds(false)
		, bFormExtendedStructures(false)
	{}
};


/** Parameters that pertain to chunk damage. */
USTRUCT()
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleDamageParameters
{
	GENERATED_USTRUCT_BODY()

	/**
		The damage amount which will cause a chunk to fracture (break free).
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleDamageParameters)
	float DamageThreshold;

	/** Controls how easily damage spreads. DamageRadius = Damage*DamageSpread. All chunks within DamageRadius will take damage. Full damage is taken at zero distance, and zero damage at the DamageRadius.*/
	UPROPERTY(EditAnywhere, Category=DestructibleDamageParameters)
	float DamageSpread;

	/**
		Whether to apply damage to destructible when colliding with an object.
		@see ImpactDamage
	*/
	UPROPERTY(EditAnywhere, Category = DestructibleDamageParameters)
	bool bEnableImpactDamage;

	/**
		Controls how much damage is applied upon collision. Damage = ImpactDamage * ImpactForce.
		@see DepthParameters for per level control of ImpactDamage
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleDamageParameters, meta = (editcondition = "bEnableImpactDamage") )
	float ImpactDamage;

	/**
		Max depth level where impact damage is enabled. @see DepthParameters for per level control of ImpactDamage
		If negative, impact damage is disabled
	*/
	UPROPERTY(EditAnywhere, Category = DestructibleDamageParameters, meta = (ClampMin = "0", UIMin = "0", editcondition = "bEnableImpactDamage"))
	int32 DefaultImpactDamageDepth;

	/**
		By default, objects that collide with destructibles will bounce back. Custom resistance allows for finer control of how much a destructible "pushes back" against a colliding object.
		@see ImpactResistance
	*/
	UPROPERTY(EditAnywhere, Category = DestructibleDamageParameters)
	bool bCustomImpactResistance;

	/**
		Controls how much resistance is applied to colliding objects. Weak materials like glass should set this to a low value so that objects will pass right through them during fracture.
		@see DepthParameters for per level control of ImpactResistance
	*/
	UPROPERTY(EditAnywhere, meta = (ClampMin = "0.001", UIMin = "0", editcondition = "bCustomImpactResistance"), Category = DestructibleDamageParameters)
	float ImpactResistance;


	FDestructibleDamageParameters()
		: DamageThreshold(1.0f)
		, DamageSpread(0.1f)
		, bEnableImpactDamage(false)
		, ImpactDamage(0.1f)
		, DefaultImpactDamageDepth(0.f)
		, bCustomImpactResistance(false)
		, ImpactResistance(1.f)
	{}
};


/** Parameters that pertain to chunk debris-level settings. */
USTRUCT()
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleDebrisParameters
{
	GENERATED_USTRUCT_BODY()

	/**
		"Debris chunks" (see debrisDepth, above) will be destroyed after a time (in seconds)
		separated from non-debris chunks.  The actual lifetime is interpolated between these
		two bDebrisTimeout, based upon the module's LOD setting.  To disable lifetime, clear the
		bDebrisTimeout flag in the flags field.
		If debrisLifetimeMax < debrisLifetimeMin, the mean of the two is used for both.
		Default debrisLifetimeMin = 1.0, debrisLifetimeMax = 10.0f.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleDebrisParameters)
	float DebrisLifetimeMin;

	UPROPERTY(EditAnywhere, Category=DestructibleDebrisParameters)
	float DebrisLifetimeMax;

	/**
		"Debris chunks" (see debrisDepth, above) will be destroyed if they are separated from
		their origin by a distance greater than maxSeparation.  The actual maxSeparation is
		interpolated between these two values, based upon the module's LOD setting.  To disable
		maxSeparation, clear the bDebrisMaxSeparation flag in the flags field.
		If debrisMaxSeparationMax < debrisMaxSeparationMin, the mean of the two is used for both.
		Default debrisMaxSeparationMin = 1.0, debrisMaxSeparationMax = 10.0f.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleDebrisParameters)
	float DebrisMaxSeparationMin;

	UPROPERTY(EditAnywhere, Category=DestructibleDebrisParameters)
	float DebrisMaxSeparationMax;

	/**
		"Debris chunks" (see debrisDepth, above) will be destroyed if they leave this box.
		The box translates with the destructible actor's initial position, but does not
		rotate or scale.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleDebrisParameters)
	FBox ValidBounds;



	FDestructibleDebrisParameters()
		: DebrisLifetimeMin(1.0f)
		, DebrisLifetimeMax(10.0f)
		, DebrisMaxSeparationMin(1.0f)
		, DebrisMaxSeparationMax(10.0f)
		, ValidBounds(FVector(-500000.0f), FVector(500000.0f))
	{}
};


/** Parameters that are less-often used. */
USTRUCT()
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleAdvancedParameters
{
	GENERATED_USTRUCT_BODY()

	/**
		Limits the amount of damage applied to a chunk.  This is useful for preventing the entire destructible
		from getting pulverized by a very large application of damage.  This can easily happen when impact damage is
		used, and the damage amount is proportional to the impact force (see forceToDamage).
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleAdvancedParameters)
	float DamageCap;

	/** 
		Large impact force may be reported if rigid bodies are spawned inside one another.  In this case the realative velocity of the two
		objects will be low.  This variable allows the user to set a minimum velocity threshold for impacts to ensure that the objects are 
		moving at a min velocity in order for the impact force to be considered.  
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleAdvancedParameters)
	float ImpactVelocityThreshold;

	/**
		If greater than 0, the chunks' speeds will not be allowed to exceed this value.  Use 0
		to disable this feature (this is the default).
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleAdvancedParameters)
	float MaxChunkSpeed;

	/**
		Scale factor used to apply an impulse force along the normal of chunk when fractured.  This is used
		in order to "push" the pieces out as they fracture.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleAdvancedParameters)
	float FractureImpulseScale;


	FDestructibleAdvancedParameters()
		: DamageCap(0)
		, ImpactVelocityThreshold(0)
		, MaxChunkSpeed(0)
		, FractureImpulseScale(0)
	{}
};


/** Special hierarchy depths for various behaviors. */
USTRUCT()
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleSpecialHierarchyDepths
{
	GENERATED_USTRUCT_BODY()

	// todo dynamicChunksDominanceGroup
	// todo dynamicChunksGroupsMask
	/**
		The chunk hierarchy depth at which to create a support graph.  Higher depth levels give more detailed support,
		but will give a higher computational load.  Chunks below the support depth will never be supported.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleSpecialHierarchyDepths)
	int32 SupportDepth;

	/**
		The chunks will not be broken free below this depth.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleSpecialHierarchyDepths)
	int32 MinimumFractureDepth;


	/** 
		Enables debris at a specific depth level.
		@see DebrisDepth */
	UPROPERTY(EditAnywhere, Category = DestructibleSpecialHierarchyDepths)
	bool bEnableDebris;

	/**
		The hierarchy depth at which chunks are considered to be "debris."  
	*/
	UPROPERTY(EditAnywhere, Category = DestructibleSpecialHierarchyDepths, meta = (UIMin = "0", editcondition = "bEnableDebris"))
	int32 DebrisDepth;

	/**
		The chunk hierarchy depth up to which chunks will always be processed.  These chunks are considered
		to be essential either for gameplay or visually.
		The minimum value is 0, meaning the level 0 chunk is always considered essential.
		Default value is 0.
	*/
	UPROPERTY(EditAnywhere, Category=DestructibleSpecialHierarchyDepths, meta=(DisplayName = "Essential LOD Depth"))
	int32 EssentialDepth;

	/**
	 * Default constructor.
	 */
	FDestructibleSpecialHierarchyDepths()
		: SupportDepth(0)
		, MinimumFractureDepth(0)
		, bEnableDebris(false)
		, DebrisDepth(-1)
		, EssentialDepth(0)
	{}
};


/** Parameters that apply to a destructible actor. */
USTRUCT()
struct UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleParameters
{
	GENERATED_USTRUCT_BODY()

	/** Parameters that pertain to chunk damage.  See DestructibleDamageParameters. */
	UPROPERTY(EditAnywhere, Category=DestructibleParameters)
	struct FDestructibleDamageParameters DamageParameters;

	/** Parameters that pertain to chunk debris-level settings.  See DestructibleDebrisParameters. */
	UPROPERTY(EditAnywhere, Category=DestructibleParameters)
	struct FDestructibleDebrisParameters DebrisParameters;

	/** Parameters that are less-often used.  See DestructibleAdvancedParameters. */
	UPROPERTY(EditAnywhere, Category=DestructibleParameters)
	struct FDestructibleAdvancedParameters AdvancedParameters;

	/** Special hierarchy depths for various behaviors. */
	UPROPERTY(EditAnywhere, Category=DestructibleParameters)
	struct FDestructibleSpecialHierarchyDepths SpecialHierarchyDepths;

	/**
		Parameters that apply to every chunk at a given level.
		the element [0] of the array applies to the level 0 (unfractured) chunk, element [1] applies
		to the level 1 chunks, etc.
	*/
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(EditAnywhere, editfixedsize, Category=DestructibleParameters)
	TArray<struct FDestructibleDepthParameters> DepthParameters;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** A collection of flags defined in DestructibleParametersFlag. */
	UPROPERTY(EditAnywhere, Category=DestructibleParameters)
	struct FDestructibleParametersFlag Flags;
};


/**
 * Holds an APEX destructible asset as well as an associated USkeletalMesh.
 */
UCLASS(hidecategories = (Object, Mesh, LevelOfDetail, Mirroring, Physics, Reimport, Clothing), MinimalAPI)
class UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") UDestructibleMesh : public USkeletalMesh
{
	GENERATED_UCLASS_BODY()

	/** Parameters controlling the destruction behavior. */
	UPROPERTY(EditAnywhere, Category=DestructibleMesh)
	struct FDestructibleParameters DefaultDestructibleParameters;

	/** Fracture effects for each fracture level, unless overridden in the component. */
	UPROPERTY(EditAnywhere, editfixedsize, Category=Effects)
	TArray<struct FFractureEffect> FractureEffects;

#if WITH_EDITORONLY_DATA
	/** Information used to author an NxDestructibleAsset*/
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(instanced)
	TObjectPtr<class UDestructibleFractureSettings> FractureSettings;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Static mesh this destructible mesh is created from. Is nullptr if not created from a static mesh */
	UPROPERTY()
	TObjectPtr<UStaticMesh> SourceStaticMesh;
	
	/** Timestamp of the source static meshes last import at the time this destruction mesh has been generated. */
	UPROPERTY()
	FDateTime SourceSMImportTimestamp;

	/** Array of static meshes to build the fracture chunks from */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UStaticMesh>> FractureChunkMeshes;
#endif // WITH_EDITORONLY_DATA

public:

	// UObject interface.

	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void FinishDestroy() override;
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;

	virtual bool HasCustomActorFactory() const
	{
		return true;
	}

	virtual bool HasCustomActorReimportFactory() const
	{
		return true;
	}
#endif

public:

	/** Fills DefaultDestructibleParameters with parameters from the NxDestructibleAsset. */
	APEXDESTRUCTION_API void LoadDefaultDestructibleParametersFromApexAsset();

	/**
	 * Create DestructibleFractureSettings for this DestructibleMesh if it doesn't have one.
	 */
	APEXDESTRUCTION_API void CreateFractureSettings();

	/**
	 * Imports FractureSettings data from a StaticMesh
	 *
	 * @param StaticMesh The StaticMesh to import
	 * @return true on success, false otherwise.
	 */
	APEXDESTRUCTION_API bool BuildFractureSettingsFromStaticMesh(UStaticMesh* StaticMesh);

	/** 
	 * Initializes this DestructibleMesh from a StaticMesh.
	 *
	 * @param StaticMesh The StaticMesh to import
	 * @return true on success, false otherwise.
	 */
	APEXDESTRUCTION_API bool BuildFromStaticMesh(UStaticMesh& StaticMesh);

	/** 
	 * Initialized this DestructibleMesh from the StaticMesh it was created from and the passed
	 * in ChunkMeshes to build the level 1 chunks from.
	 *
	 * @param ChunkMeshes Meshes to build the level1 chunks from
	 * @return true on success, false otherwise.
	 */
	APEXDESTRUCTION_API bool SetupChunksFromStaticMeshes(const TArray<UStaticMesh*>& ChunkMeshes);
};

APEXDESTRUCTION_API DECLARE_LOG_CATEGORY_EXTERN(LogDestructible, Warning, All);