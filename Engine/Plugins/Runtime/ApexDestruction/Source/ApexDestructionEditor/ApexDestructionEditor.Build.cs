// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ApexDestructionEditor : ModuleRules
{
	public ApexDestructionEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject", // @todo Mac: for some reason it's needed to link in debug on Mac
				"Engine",
                "ApexDestruction",
                "AssetTools",
//                "PhysX",
//                "APEX",
                "InputCore",
                "RenderCore",
                "RHI",
                "Slate",
                "SlateCore",
                "EditorStyle",
				"EditorFramework",
                "UnrealEd",
                "DesktopPlatform",
                "AssetRegistry",
                "ContentBrowser",
                "Projects",
//                "ApexDestructionLib"
            }
		);

        AddEngineThirdPartyPrivateStaticDependencies(Target,
            "FBX"
        );

        SetupModulePhysicsSupport(Target);
}
}
