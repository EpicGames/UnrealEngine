// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactoryDestructible.h"
#include "DestructibleActor.h"
#include "DestructibleMesh.h"
#include "DestructibleComponent.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "ActorFactoryDestructible"

DEFINE_LOG_CATEGORY_STATIC(LogDestructibleFactories, Log, All);

/*-----------------------------------------------------------------------------
UActorFactoryDestructible
-----------------------------------------------------------------------------*/
UActorFactoryDestructible::UActorFactoryDestructible(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NewActorClass = ADestructibleActor::StaticClass();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	DisplayName = LOCTEXT("DestructibleDisplayName", "Destructible");
	bUseSurfaceOrientation = true;
}

bool UActorFactoryDestructible::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!AssetData.IsValid() || !AssetData.GetClass()->IsChildOf(UDestructibleMesh::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoDestructibleMeshSpecified", "No destructible mesh was specified.");
		return false;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return true;
}

void UActorFactoryDestructible::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDestructibleMesh* DestructibleMesh = CastChecked<UDestructibleMesh>(Asset);
	ADestructibleActor* NewDestructibleActor = CastChecked<ADestructibleActor>(NewActor);

	// Term Component
	NewDestructibleActor->GetDestructibleComponent()->UnregisterComponent();

	// Change properties
	NewDestructibleActor->GetDestructibleComponent()->SetSkeletalMesh(DestructibleMesh);

	// Init Component
	NewDestructibleActor->GetDestructibleComponent()->RegisterComponent();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UObject* UActorFactoryDestructible::GetAssetFromActorInstance(AActor* Instance)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(Instance->IsA(NewActorClass));
	ADestructibleActor* DA = CastChecked<ADestructibleActor>(Instance);

	check(DA->GetDestructibleComponent());
	return DA->GetDestructibleComponent()->SkeletalMesh;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FQuat UActorFactoryDestructible::AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const
{
	// Meshes align the Z (up) axis with the surface normal
	return FindActorAlignmentRotation(ActorRotation, FVector(0.f, 0.f, 1.f), InSurfaceNormal);
}

#undef LOCTEXT_NAMESPACE
