// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApexDestructionEditorModule.h"
#include "AssetTypeActions_DestructibleMesh.h"
#include "DestructibleMeshThumbnailRenderer.h"
#include "DestructibleMeshComponentBroker.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
IMPLEMENT_MODULE( FDestructibleMeshEditorModule, ApexDestructionEditor );
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#define LOCTEXT_NAMESPACE "DestructibleMeshEditor"

void FDestructibleMeshEditorModule::StartupModule()
{

	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

	AssetAction = MakeShareable(new FAssetTypeActions_DestructibleMesh);
	AssetTools.RegisterAssetTypeActions(AssetAction.ToSharedRef());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UThumbnailManager::Get().RegisterCustomRenderer(UDestructibleMesh::StaticClass(), UDestructibleMeshThumbnailRenderer::StaticClass());

	DestructibleMeshComponentBroker = MakeShareable(new FDestructibleMeshComponentBroker);
	FComponentAssetBrokerage::RegisterBroker(DestructibleMeshComponentBroker, UDestructibleComponent::StaticClass(), false, true);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FDestructibleMeshEditorModule::ShutdownModule()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.UnregisterAssetTypeActions(AssetAction.ToSharedRef());
	}

	if (UObjectInitialized())
	{
		UThumbnailManager::Get().UnregisterCustomRenderer(UDestructibleMesh::StaticClass());
		FComponentAssetBrokerage::UnregisterBroker(DestructibleMeshComponentBroker);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#undef LOCTEXT_NAMESPACE
