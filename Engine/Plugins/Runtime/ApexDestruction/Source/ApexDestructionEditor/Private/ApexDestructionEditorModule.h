// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
//#include "Toolkits/IToolkitHost.h"
//#include "Toolkits/AssetEditorToolkit.h"
//#include "AssetData.h"

//class UStaticMesh;
//class IDestructibleMeshEditor;
//class FExtender;

class IAssetTypeActions;


//extern const FName DestructibleMeshEditorAppIdentifier;

//class IDestructibleMeshEditor;

/** DestructibleMesh Editor module */

class UE_DEPRECATED(4.26, "APEX is deprecated. Destruction in future will be supported using Chaos Destruction.") FDestructibleMeshEditorModule : public IModuleInterface
{

public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

//	PRAGMA_DISABLE_DEPRECATION_WARNINGS
//	/**
//	 * Creates an instance of table editor object.  Only virtual so that it can be called across the DLL boundary.
//	 *
//	 * @param	Mode					Mode that this editor should operate in
//	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
//	 * @param	Table					The table to start editing
//	 *
//	 * @return	Interface to the new table editor
//	 */
//	virtual TSharedRef<IDestructibleMeshEditor> CreateDestructibleMeshEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UDestructibleMesh* Table );
//
//	/**
//	 * Creates a DestructibleMesh from a StaticMesh
//	 *
//	 * @param InParent - parent object for the UDestructibleMesh
//	 * @param StaticMesh - the StaticMesh to convert
//	 * @param Name - the Unreal name for the UDestructibleMesh.  If Name == NAME_None, then a the StaticMesh name followed appended with "_Dest" will be used.
//	 * @param Flags - object flags for the UDestructibleMesh
//	 * @param OutErrorMsg - if it returns false, this string contains the error
//	 *
//	 * @return The newly created UDestructibleMesh if successful, NULL otherwise
//	 */
//	virtual class UDestructibleMesh*	CreateDestructibleMeshFromStaticMesh(UObject* InParent, UStaticMesh* StaticMesh, FName Name, EObjectFlags Flags, FText& OutErrorMsg);
//	PRAGMA_ENABLE_DEPRECATION_WARNINGS
//
//	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
//	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override {return MenuExtensibilityManager;}
//	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override {return ToolBarExtensibilityManager;}
//
//private:
//	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
//	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<class IAssetTypeActions> AssetAction;
	TSharedPtr<class FDestructibleMeshComponentBroker> DestructibleMeshComponentBroker;
//	FDelegateHandle ContentBrowserExtenderDelegateHandle;
//
//private:
//	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets);
};


