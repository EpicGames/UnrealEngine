// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_DestructibleMesh.h"
#include "DestructibleMesh.h"
//#include "ApexDestructionEditorModule.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UClass* FAssetTypeActions_DestructibleMesh::GetSupportedClass() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return UDestructibleMesh::StaticClass();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FAssetTypeActions_DestructibleMesh::OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor )
{
//	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
//	{
//PRAGMA_DISABLE_DEPRECATION_WARNINGS
//		auto Mesh = Cast<UDestructibleMesh>(*ObjIt);
//
//		if (Mesh != NULL)
//		{
//			FDestructibleMeshEditorModule& DestructibleMeshEditorModule = FModuleManager::LoadModuleChecked<FDestructibleMeshEditorModule>( "ApexDestructionEditor" );
//			TSharedRef< IDestructibleMeshEditor > NewDestructibleMeshEditor = DestructibleMeshEditorModule.CreateDestructibleMeshEditor( EToolkitMode::Standalone, EditWithinLevelEditor, Mesh );
//		}
//PRAGMA_ENABLE_DEPRECATION_WARNINGS
//	}
}

#undef LOCTEXT_NAMESPACE
