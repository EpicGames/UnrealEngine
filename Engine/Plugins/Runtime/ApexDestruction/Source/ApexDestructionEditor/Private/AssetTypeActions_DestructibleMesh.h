// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

class FAssetTypeActions_DestructibleMesh : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DestructibleMesh", "Destructible Mesh"); }
	virtual FColor GetTypeColor() const override { return FColor(200,128,128); }
	virtual UClass* GetSupportedClass() const override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Physics; }
	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) override;
	virtual bool IsImportedAsset() const override { return false; }
};
