// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentAssetBroker.h"
#include "DestructibleMesh.h"
#include "DestructibleComponent.h"

//////////////////////////////////////////////////////////////////////////
// FDestructibleMeshComponentBroker
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FDestructibleMeshComponentBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return UDestructibleMesh::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (UDestructibleComponent* DestMeshComp = Cast<UDestructibleComponent>(InComponent))
		{
			UDestructibleMesh* DMesh = Cast<UDestructibleMesh>(InAsset);

			if (DMesh)
			{
				DestMeshComp->SetDestructibleMesh(DMesh);
				return true;
			}
		}

		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (UDestructibleComponent* DestMeshComp = Cast<UDestructibleComponent>(InComponent))
		{
			return DestMeshComp->GetDestructibleMesh();
		}
		return NULL;
	}
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
