// Copyright Epic Games, Inc. All Rights Reserved.

#include "DestructibleMeshThumbnailRenderer.h"
#include "SceneView.h"

// FPreviewScene derived helpers for rendering
#include "DestructibleMesh.h"
#include "DestructibleMeshThumbnailScene.h"

UDestructibleMeshThumbnailRenderer::UDestructibleMeshThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ThumbnailScene = nullptr;
}

void UDestructibleMeshThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UDestructibleMesh* DestructibleMesh = Cast<UDestructibleMesh>(Object);
	if (DestructibleMesh != nullptr)
	{
		if ( ThumbnailScene == nullptr )
		{
			ThumbnailScene = new FDestructibleMeshThumbnailScene();
		}

		ThumbnailScene->SetDestructibleMesh(DestructibleMesh);
		FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->SetDestructibleMesh(nullptr);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDestructibleMeshThumbnailRenderer::BeginDestroy()
{
	if ( ThumbnailScene != nullptr )
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}
