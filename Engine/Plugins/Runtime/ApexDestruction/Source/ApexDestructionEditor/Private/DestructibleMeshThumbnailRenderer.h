// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given skeletal mesh
 */

#pragma once

#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "DestructibleMeshThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS(config=Editor, MinimalAPI)
class UDestructibleMeshThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()


	// Begin UThumbnailRenderer Object
	APEXDESTRUCTIONEDITOR_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// UObject implementation
	APEXDESTRUCTIONEDITOR_API virtual void BeginDestroy() override;

private:
	class FDestructibleMeshThumbnailScene* ThumbnailScene;
};

