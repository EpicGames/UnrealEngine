// Copyright Epic Games, Inc. All Rights Reserved.

#include "DestructibleMeshThumbnailScene.h"
#include "DestructibleActor.h"
#include "DestructibleComponent.h"
#include "DestructibleMesh.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"

/*
***************************************************************
FDestructibleMeshThumbnailScene
***************************************************************
*/

FDestructibleMeshThumbnailScene::FDestructibleMeshThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;
	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PreviewActor = GetWorld()->SpawnActor<ADestructibleActor>(SpawnInfo);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PreviewActor->SetActorEnableCollision(false);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDestructibleMeshThumbnailScene::SetDestructibleMesh(UDestructibleMesh* InMesh)
{
	PreviewActor->GetDestructibleComponent()->OverrideMaterials.Empty();
	PreviewActor->GetDestructibleComponent()->SetDestructibleMesh(InMesh);

	if (InMesh)
	{
		FTransform MeshTransform = FTransform::Identity;

		PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
		PreviewActor->GetDestructibleComponent()->UpdateBounds();

		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetDestructibleComponent()->Bounds);
		PreviewActor->SetActorLocation(-PreviewActor->GetDestructibleComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false);
		PreviewActor->GetDestructibleComponent()->RecreateRenderState_Concurrent();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FDestructibleMeshThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(PreviewActor->GetDestructibleComponent());
	check(PreviewActor->GetDestructibleComponent()->GetDestructibleMesh());
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// No need to add extra size to view slightly outside of the sphere to compensate for perspective since skeletal meshes already buffer bounds.
	const float HalfMeshSize = PreviewActor->GetDestructibleComponent()->Bounds.SphereRadius;
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetDestructibleComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewActor->GetDestructibleComponent()->DestructibleMesh_DEPRECATED->GetThumbnailInfo());
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}
