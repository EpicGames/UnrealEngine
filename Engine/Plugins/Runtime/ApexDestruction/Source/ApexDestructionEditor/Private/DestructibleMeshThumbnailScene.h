// Copyright Epic Games, Inc. All Rights Reserved.

/*
***************************************************************
FDestructibleMeshThumbnailScene
***************************************************************
*/

#pragma once

#include "ThumbnailHelpers.h"

class APEXDESTRUCTIONEDITOR_API FDestructibleMeshThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	FDestructibleMeshThumbnailScene();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Sets the skeletal mesh to use in the next CreateView() */
	void SetDestructibleMesh(class UDestructibleMesh* InMesh);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	// FThumbnailPreviewScene implementation
	virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** The skeletal mesh actor used to display all skeletal mesh thumbnails */
	class ADestructibleActor* PreviewActor;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};